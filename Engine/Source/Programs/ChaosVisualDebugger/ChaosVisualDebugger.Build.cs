// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosVisualDebugger : ModuleRules
{
	public ChaosVisualDebugger(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("ChaosVD");

		// LaunchEngineLoop dependencies
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"InstallBundleManager",
				"MediaUtils",
				"Messaging",
				"MoviePlayer",
				"MoviePlayerProxy",
				"Projects",
				"PreLoadScreen",
				"RenderCore",
				"RHI",
				"Slate",
				"SlateCore",
				"TraceLog",
				"ProfileVisualizer",
				"PropertyAccessEditor"
			}
		);

		// LaunchEngineLoop IncludePath dependencies
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
				"AutomationWorker",
				"AutomationController",
				"AutomationTest",
				"DerivedDataCache",
				"HeadMountedDisplay", 
				"MRMesh", 
				"SlateRHIRenderer", 
				"SlateNullRenderer",
			}
		);

		// LaunchEngineLoop editor dependencies
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				
				"PropertyEditor",
				"DerivedDataCache",
				"ToolWidgets",
				"UnrealEd"
			});

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnixCommonStartup"
				}
			);
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PrivateDependencyModuleNames.Add("AgilitySDK");
		}
	}
}
