// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms("Win64")]
public class ChaosVisualDebuggerTarget : TargetRules
{
	public ChaosVisualDebuggerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "ChaosVisualDebugger";
		BuildEnvironment = TargetBuildEnvironment.Unique;

		DefaultBuildSettings = BuildSettingsVersion.Latest;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		
		bIsBuildingConsoleApplication = false;
		
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = true;
		bCompileAgainstEditor = true;
		bBuildWithEditorOnlyData = true;
		bUsesSlate = true;
		bEnableTrace = true;
		bUseLoggingInShipping = true;

		bCompileWithPluginSupport = true;
		bBuildDeveloperTools = true;
		
		// Disabled whatever we can and we don't need
		bCompilePython = false;
		bCompileICU = true;
		bCompileRecast = false;
		bUseGameplayDebugger = false;
		bCompileNavmeshClusterLinks = false;
		bCompileNavmeshSegmentLinks = false;
		bWithLiveCoding = false;
		bIncludePluginsForTargetPlatforms = false;

		bCompileChaosVisualDebuggerSupport = true;
		
		OptedInModulePlatforms = new UnrealTargetPlatform[] { UnrealTargetPlatform.Win64 };

		AdditionalPlugins.AddRange(new string[]
		{
			"ChaosVD",
			"GeometryProcessing",
			"PropertyAccessEditor"
		});

		EnablePlugins.Add("GeometryProcessing");
		EnablePlugins.Add("ChaosVD");
		EnablePlugins.Add("PropertyAccessEditor");

		GlobalDefinitions.Add("AUTOSDKS_ENABLED=0");
		GlobalDefinitions.Add("ASSETREGISTRY_FORCE_PREMADE_REGISTRY_IN_EDITOR=1");
		GlobalDefinitions.Add("UE_IS_COOKED_EDITOR=1");
		GlobalDefinitions.Add("UE_FORCE_USE_PAKS=1");
		GlobalDefinitions.Add("UE_FORCE_USE_IOSTORE=0");

		// this will allow shader compiling to work based on whether or not the shaders directory is present
		// to determine if we should allow shader compilation
		GlobalDefinitions.Add("UE_ALLOW_SHADER_COMPILING_BASED_ON_SHADER_DIRECTORY_EXISTENCE=1");
		
		GlobalDefinitions.Add("UE_TRACE_SERVER_LAUNCH_ENABLED=1");
		GlobalDefinitions.Add("UE_TRACE_SERVER_CONTROLS_ENABLED=1");
	}
}
