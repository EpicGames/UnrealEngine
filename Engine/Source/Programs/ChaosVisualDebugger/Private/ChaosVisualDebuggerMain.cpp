// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVisualDebuggerMain.h"

#include "RequiredProgramMainCPPInclude.h"
#include "EditorViewportClient.h"
#include "LaunchEngineLoop.h"
#include "HAL/PlatformSplash.h"

IMPLEMENT_APPLICATION(ChaosVisualDebugger, "ChaosVisualDebugger");

DEFINE_LOG_CATEGORY_STATIC(LogChaosVisualDebugger, Log, All);

#if !defined(D3D12_CORE_ENABLED)
	#define D3D12_CORE_ENABLED 0
#endif

#if D3D12_CORE_ENABLED
// Add Agility SDK symbols for Windows
#include "UnrealAgilitySDKLink.inl"
#endif

int32 RunChaosVisualDebugger(const TCHAR* CommandLine)
{
	FTaskTagScope Scope(ETaskTag::EGameThread);

#if !(UE_BUILD_SHIPPING)

	// If "-waitforattach" or "-WaitForDebugger" was specified, halt startup and wait for a debugger to attach before continuing
	if (FParse::Param(CommandLine, TEXT("waitforattach")) || FParse::Param(CommandLine, TEXT("WaitForDebugger")))
	{
		while (!FPlatformMisc::IsDebuggerPresent())
		{
			FPlatformProcess::Sleep(0.1f);
		}
		UE_DEBUG_BREAK();
	}

#endif

	// Override the stack size for the thread pool.
	FQueuedThreadPool::OverrideStackSize = 256 * 1024;

	const FText AppName = NSLOCTEXT("ChaosVisualDebugger", "ChaosVisualDebuggerSplashText", "Chaos Visual Debugger");
	FPlatformSplash::SetSplashText(SplashTextType::GameName, AppName);

	FString Command;
	const bool bIsRunningCommand = FParse::Value(CommandLine, TEXT("-RUN="), Command);

	const FString CommandLineString = CommandLine;
	const FString FinalCommandLine = bIsRunningCommand ? CommandLine : CommandLineString + TEXT(" EDITOR -messaging");

	// start up the main loop
	const int32 Result = GEngineLoop.PreInit(*FinalCommandLine);

	if (Result != 0)
	{
		UE_LOGF(LogChaosVisualDebugger, Error, "EngineLoop PreInit failed!");
		return Result;
	}

	if (!bIsRunningCommand)
	{
		// Register navigation commands for all viewports
		FViewportNavigationCommands::Register();

		GEngineLoop.Init();

		// Hide the splash screen now that everything is ready to go
		FPlatformSplash::Hide();

		while (!IsEngineExitRequested())
		{
			GEngineLoop.Tick();
		}
	}

	GEngineLoop.Exit();

	return Result;
}